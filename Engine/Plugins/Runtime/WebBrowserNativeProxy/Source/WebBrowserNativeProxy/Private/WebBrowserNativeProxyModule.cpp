// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebBrowserNativeProxyModule.h"
#include "WebBrowserModule.h"
#include "IWebBrowserWindow.h"
#include "IWebBrowserPopupFeatures.h"
#include "SWebBrowser.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Misc/App.h"
#include "Widgets/SWindow.h"

class FWebBrowserNativeProxyModule : public IWebBrowserNativeProxyModule
{
public:
	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
		Browser.Reset();
	}

	virtual TSharedPtr<IWebBrowserWindow> GetBrowser(bool bCreate) override
	{
		if (bCreate && FApp::CanEverRender())
		{
			CreateBrowser();
		}
		return Browser;
	}

	virtual FOnBrowserAvailableEvent& OnBrowserAvailable() override
	{
		return BrowserAvailableEvent;
	}
	
private:

	void CreateBrowser()
	{
		if (!Browser.IsValid())
		{			
#if BUILD_EMBEDDED_APP
			Browser = IWebBrowserModule::Get().GetSingleton()->CreateNativeBrowserProxy();
#else

			FCreateBrowserWindowSettings WindowSettings;
			WindowSettings.bUseTransparency = true;
			WindowSettings.bInterceptLoadRequests = true;
			WindowSettings.bShowErrorMessage = false;
			Browser = IWebBrowserModule::Get().GetSingleton()->CreateBrowserWindow(WindowSettings);

#if !UE_BUILD_SHIPPING
			IWebBrowserModule::Get().GetSingleton()->SetDevToolsShortcutEnabled(true);
			Browser->OnCreateWindow().BindRaw(this, &FWebBrowserNativeProxyModule::HandleBrowserCreateWindow);
#endif
#endif
			OnBrowserAvailable().Broadcast(Browser.ToSharedRef());
		}
	}

#if !BUILD_EMBEDDED_APP && !UE_BUILD_SHIPPING
	bool HandleBrowserCreateWindow(const TWeakPtr<IWebBrowserWindow>& NewBrowserWindow, const TWeakPtr<IWebBrowserPopupFeatures>& PopupFeatures)
	{
		TSharedPtr<SWindow> ParentDebugToolsWindow = FSlateApplication::Get().GetActiveTopLevelWindow();
		if (ParentDebugToolsWindow.IsValid())
		{
			TSharedPtr<IWebBrowserPopupFeatures> PopupFeaturesSP = PopupFeatures.Pin();
			check(PopupFeatures.IsValid())

			const int PosX = PopupFeaturesSP->IsXSet() ? PopupFeaturesSP->GetX() : 100;
			const int PosY = PopupFeaturesSP->IsYSet() ? PopupFeaturesSP->GetY() : 100;
			const FVector2D BrowserWindowPosition(PosX, PosY);

			const int Width = PopupFeaturesSP->IsWidthSet() ? PopupFeaturesSP->GetWidth() : 1024;
			const int Height = PopupFeaturesSP->IsHeightSet() ? PopupFeaturesSP->GetHeight() : 768;
			const FVector2D BrowserWindowSize(Width, Height);

			const ESizingRule SizeingRule = PopupFeaturesSP->IsResizable() ? ESizingRule::UserSized : ESizingRule::FixedSize;

			TSharedPtr<IWebBrowserWindow> NewBrowserWindowSP = NewBrowserWindow.Pin();
			check(NewBrowserWindowSP.IsValid())

			TSharedRef<SWindow> BrowserWindowWidget =
				SNew(SWindow)
				.Title(FText::FromString(TEXT("Debug")))
				.ClientSize(BrowserWindowSize)
				.ScreenPosition(BrowserWindowPosition)
				.AutoCenter(EAutoCenter::None)
				.SizingRule(SizeingRule)
				.SupportsMaximize(SizeingRule != ESizingRule::FixedSize)
				.SupportsMinimize(SizeingRule != ESizingRule::FixedSize)
				.HasCloseButton(true)
				.CreateTitleBar(true)
				.IsInitiallyMaximized(PopupFeaturesSP->IsFullscreen())
				.LayoutBorder(FMargin(0));

			// Setup browser widget.
			TSharedPtr<SWebBrowser> BrowserWidget;
			BrowserWindowWidget->SetContent(
				SNew(SBorder)
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				.Padding(0)
				[
					SAssignNew(BrowserWidget, SWebBrowser, NewBrowserWindowSP)
					.ShowControls(PopupFeaturesSP->IsToolBarVisible())
					.ShowAddressBar(PopupFeaturesSP->IsLocationBarVisible())
				]
			);

			// Setup some OnClose stuff.
			{
				struct FLocal
				{
					static void RequestDestroyWindowOverride(const TSharedRef<SWindow>& Window, TWeakPtr<IWebBrowserWindow> BrowserWindowPtr)
					{
						TSharedPtr<IWebBrowserWindow> BrowserWindow = BrowserWindowPtr.Pin();
						if (BrowserWindow.IsValid())
						{
							if (BrowserWindow->IsClosing())
							{
								FSlateApplicationBase::Get().RequestDestroyWindow(Window);
							}
							else
							{
								BrowserWindow->CloseBrowser(false);
							}
						}
					}
				};

				BrowserWindowWidget->SetRequestDestroyWindowOverride(FRequestDestroyWindowOverride::CreateStatic(&FLocal::RequestDestroyWindowOverride, TWeakPtr<IWebBrowserWindow>(NewBrowserWindow)));
			}

			FSlateApplication::Get().AddWindowAsNativeChild(BrowserWindowWidget, ParentDebugToolsWindow.ToSharedRef());
			BrowserWindowWidget->BringToFront();
			FSlateApplication::Get().SetKeyboardFocus(BrowserWidget, EFocusCause::SetDirectly);

			return true;
		}
		return false;
	}
#endif

	TSharedPtr<IWebBrowserWindow> Browser;
	FOnBrowserAvailableEvent BrowserAvailableEvent;
};


IMPLEMENT_MODULE(FWebBrowserNativeProxyModule, WebBrowserNativeProxy);
