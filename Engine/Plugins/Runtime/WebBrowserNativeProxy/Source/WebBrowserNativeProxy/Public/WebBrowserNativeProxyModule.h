// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

class IWebBrowserWindow;

/**
 * The public interface to this module
 */
class IWebBrowserNativeProxyModule : public IModuleInterface
{

public:

	/**
	 * Name of this module
	 */
	static inline const TCHAR* GetModuleName()
	{
		return TEXT("WebBrowserNativeProxy");
	}

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IWebBrowserNativeProxyModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IWebBrowserNativeProxyModule>(GetModuleName());
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(GetModuleName());
	}

public:

	/**
	 * Get the single browser window associated with the module
	 *
	 * @param bCreate if true then create the window if not already created
	 * @return ptr to the browser window
	 */
	virtual TSharedPtr<IWebBrowserWindow> GetBrowser(bool bCreate=false) = 0;

	/**
	 * Callback for when the browser window has been created and is available
	 */
	DECLARE_EVENT_OneParam(IWebBrowserNativeProxyModule, FOnBrowserAvailableEvent, const TSharedRef<IWebBrowserWindow>& /*Browser*/);
	virtual FOnBrowserAvailableEvent& OnBrowserAvailable() = 0;
};
