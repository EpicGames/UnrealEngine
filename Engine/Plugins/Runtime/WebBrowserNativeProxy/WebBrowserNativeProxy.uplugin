{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Web Browser to Native Proxying",
	"Description": "Maintains the browser to native proxy and provides hooks for registering UObjects bindings",
	"Category": "UI",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "WebBrowserNativeProxy",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault",
			"PlatformAllowList": [
				"Win64",
				"Mac",
				"IOS",
				"Android"
			]
		}
	],
	"CanContainContent": false,
	"IsExperimentalVersion": false
}
